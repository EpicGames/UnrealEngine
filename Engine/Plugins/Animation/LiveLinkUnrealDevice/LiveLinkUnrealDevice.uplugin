{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Live Link Hub Unreal Device",
	"Description": "Live Link Hub device providing Take Recorder automation. Must be enabled in both Live Link Hub and Unreal Editor.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "LiveLinkDevice",
			"Enabled": true
		},
		{
			"Name": "LiveLinkHub",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "LiveLinkUnrealDevice",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
