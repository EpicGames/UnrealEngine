// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LiveLinkUnrealDevice : ModuleRules
{
	public LiveLinkUnrealDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"LiveLink",
				"LiveLinkDevice",
				"LiveLinkHub",
				"LiveLinkHubMessaging",
				"LiveLinkSequencer",
				"Messaging",
				"Networking",
				"Projects",
				"Slate",
				"SlateCore",
				"TakeRecorder",
				"TakesCore",
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
