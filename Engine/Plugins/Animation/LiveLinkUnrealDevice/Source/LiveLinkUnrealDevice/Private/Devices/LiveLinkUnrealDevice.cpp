// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkUnrealDevice.h"
#include "Engine/Engine.h"
#include "LiveLinkHub/ILiveLinkRecordingSession.h"
#include "INetworkMessagingExtension.h"
#include "LiveLinkDeviceSubsystem.h"
#include "LiveLinkUnrealDeviceMessages.h"
#include "Logging/StructuredLog.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"


DEFINE_LOG_CATEGORY(LogLiveLinkUnrealDevice);


#define LOCTEXT_NAMESPACE "LiveLinkUnrealDevice"


EDeviceHealth ULiveLinkUnrealDevice::GetDeviceHealth() const
{
	return GetHealth().Key;
}


FText ULiveLinkUnrealDevice::GetHealthText() const
{
	return GetHealth().Value;
}


TPair<EDeviceHealth, FText> ULiveLinkUnrealDevice::GetHealth() const
{
	const ULiveLinkUnrealDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkUnrealDeviceSettings>();
	if (!DeviceSettings->ClientId.IsValid())
	{
		return { EDeviceHealth::Info, LOCTEXT("DeviceHealthText_ClientUnset", "Client unset") };
	}
	else if (!MessageEndpoint || !DeviceAddress.IsValid())
	{
		return { EDeviceHealth::Warning, LOCTEXT("DeviceHealthText_NotConnected", "Not connected") };
	}

	return { EDeviceHealth::Good, FText::GetEmpty() };
}


void ULiveLinkUnrealDevice::OnDeviceAdded()
{
	ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
	Session.OnSlateNameChanged().AddUObject(this, &ULiveLinkUnrealDevice::HandleSlateNameChanged);
	Session.OnTakeNumberChanged().AddUObject(this, &ULiveLinkUnrealDevice::HandleTakeNumberChanged);

	Connect();
}


void ULiveLinkUnrealDevice::OnDeviceRemoved()
{
	Disconnect();
}


void ULiveLinkUnrealDevice::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::OnSettingChanged(InPropertyChangedEvent);

	const ULiveLinkUnrealDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkUnrealDeviceSettings>();

	const FName ChangedPropertyName = InPropertyChangedEvent.GetPropertyName();

	static const FName ClientIdName = GET_MEMBER_NAME_CHECKED(ULiveLinkUnrealDeviceSettings, ClientId);
	if (ChangedPropertyName == ClientIdName)
	{
		Disconnect();
		Connect();
	}
}


ELiveLinkDeviceConnectionStatus ULiveLinkUnrealDevice::GetConnectionStatus_Implementation() const
{
	return ConnectionStatus;
}


FString ULiveLinkUnrealDevice::GetHardwareId_Implementation() const
{
	const ULiveLinkUnrealDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkUnrealDeviceSettings>();
	if (DeviceSettings->ClientId.IsValid())
	{
		ILiveLinkHubClientsModel& ClientsModel = ILiveLinkHubClientsModel::GetChecked();
		return ClientsModel.GetClientDisplayName(DeviceSettings->ClientId).ToString();
	}

	return FString();
}


bool ULiveLinkUnrealDevice::SetHardwareId_Implementation(const FString& InHardwareID)
{
	return false;
}


bool ULiveLinkUnrealDevice::Connect_Implementation()
{
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Connected)
	{
		return false;
	}

	Connect();

	return true;
}


bool ULiveLinkUnrealDevice::Disconnect_Implementation()
{
	if (ConnectionStatus == ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		return false;
	}

	Disconnect();

	return true;
}

void ULiveLinkUnrealDevice::SetConnectionStatus(ELiveLinkDeviceConnectionStatus InStatus)
{
	ConnectionStatus = InStatus;
	ILiveLinkDeviceCapability_Connection::SetConnectionStatus(InStatus);
}


bool ULiveLinkUnrealDevice::StartRecording_Implementation()
{
	if (bIsRecording)
	{
		return false;
	}

	if (MessageEndpoint && DeviceAddress.IsValid())
	{
		FLiveLinkTakeRecorderCmd_StartRecording* Message = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderCmd_StartRecording>();

		ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
		Message->SlateInfo = {
			.SlateName = Session.GetSlateName(),
			.TakeNumber = Session.GetTakeNumber(),
			.Description = Session.GetSessionName(),
		};
		MessageEndpoint->Send(Message, EMessageFlags::Reliable, {}, nullptr, { DeviceAddress }, FTimespan::Zero(), FDateTime::MaxValue());
		return true;
	}

	return false;
}


bool ULiveLinkUnrealDevice::StopRecording_Implementation()
{
	if (!bIsRecording)
	{
		return false;
	}

	if (MessageEndpoint && DeviceAddress.IsValid())
	{
		MessageEndpoint->Send(FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderCmd_StopRecording>(),
			EMessageFlags::Reliable, {}, nullptr, { DeviceAddress }, FTimespan::Zero(), FDateTime::MaxValue());
		return true;
	}

	return false;
}


bool ULiveLinkUnrealDevice::IsRecording_Implementation() const
{
	return bIsRecording;
}


void ULiveLinkUnrealDevice::HandleSlateNameChanged(FStringView InSlateName)
{
	if (bSuppressSlateNameSend)
	{
		return;
	}

	// Authoritative editors own their take metadata; the Hub must not push
	// slate changes back to them.
	if (GetSettings()->bHasTakeMetadataAuthority)
	{
		return;
	}

	if (MessageEndpoint && DeviceAddress.IsValid())
	{
		FLiveLinkTakeRecorderCmd_SetSlateName* Message = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderCmd_SetSlateName>();
		Message->SlateName = InSlateName;
		MessageEndpoint->Send(Message, EMessageFlags::Reliable, {}, nullptr, { DeviceAddress }, FTimespan::Zero(), FDateTime::MaxValue());
	}
}


void ULiveLinkUnrealDevice::HandleTakeNumberChanged(int32 InTakeNumber)
{
	if (bSuppressTakeNumberSend)
	{
		return;
	}

	// Authoritative editors own their take metadata; the Hub must not push
	// take-number changes back to them (otherwise the Hub's post-stop
	// auto-increment double-bumps the editor's already-incremented take).
	if (GetSettings()->bHasTakeMetadataAuthority)
	{
		return;
	}

	if (MessageEndpoint && DeviceAddress.IsValid())
	{
		FLiveLinkTakeRecorderCmd_SetTakeNumber* Message = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderCmd_SetTakeNumber>();
		Message->TakeNumber = InTakeNumber;
		MessageEndpoint->Send(Message, EMessageFlags::Reliable, {}, nullptr, { DeviceAddress }, FTimespan::Zero(), FDateTime::MaxValue());
	}
}


void ULiveLinkUnrealDevice::HandleMessageCatchall(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext)
{
	const UScriptStruct* MessageTypeInfo = InContext->GetMessageTypeInfo().Get();

	if (MessageTypeInfo->IsChildOf(FLiveLinkHubAuxChannelBaseMessage::StaticStruct()))
	{
		const FLiveLinkHubAuxChannelBaseMessage* AuxBaseMessage =
			static_cast<const FLiveLinkHubAuxChannelBaseMessage*>(InContext->GetMessage());

		if (MessageTypeInfo->IsChildOf(FLiveLinkHubAuxChannelAcceptMessage::StaticStruct()))
		{
			ensure(AuxBaseMessage->ChannelId == ChannelId);

			DeviceAddress = InContext->GetSender();

			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Connection accepted by {RemoteAddress}",
				GetSettings()->DisplayName, DeviceAddress.ToString());

			SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connected);
		}
		else if (MessageTypeInfo->IsChildOf(FLiveLinkHubAuxChannelRejectMessage::StaticStruct()))
		{
			ensure(AuxBaseMessage->ChannelId == ChannelId);

			UE_LOGFMT(LogLiveLinkUnrealDevice, Error, "{DeviceName}: Connection rejected by {RemoteAddress}",
				GetSettings()->DisplayName, InContext->GetSender().ToString());

			OnDisconnect();
		}
		else if (MessageTypeInfo->IsChildOf(FLiveLinkHubAuxChannelCloseMessage::StaticStruct()))
		{
			ensure(AuxBaseMessage->ChannelId == ChannelId);

			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Connection closed by {RemoteAddress}",
				GetSettings()->DisplayName, InContext->GetSender().ToString());

			OnDisconnect();
		}
	}
	else if (MessageTypeInfo->IsChildOf(FLiveLinkTakeRecorderMessageBase::StaticStruct()))
	{
		if (MessageTypeInfo == FLiveLinkTakeRecorderEvent_RecordingStarted::StaticStruct())
		{
			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: RecordingStarted event from {RemoteAddress}",
				GetSettings()->DisplayName, InContext->GetSender().ToString());

			bIsRecording = true;

			if (GetDeviceSettings<ULiveLinkUnrealDeviceSettings>()->bHasRecordStartAuthority)
			{
				ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
				const bool bWasRecording = Session.IsRecording();
				const bool bCanRecord = Session.CanRecord();
				bool bStartedRecording = false;
				if (!bWasRecording && bCanRecord)
				{
					bStartedRecording = Session.StartRecording();
				}

				UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Has record start authority; "
					"bWasRecording={bWasRecording}, bCanRecord={bCanRecord}, bStartedRecording={bStartedRecording}",
					GetSettings()->DisplayName, bWasRecording, bCanRecord, bStartedRecording);
			}
		}
		else if (MessageTypeInfo == FLiveLinkTakeRecorderEvent_RecordingStopped::StaticStruct())
		{
			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: RecordingStopped event from {RemoteAddress}",
				GetSettings()->DisplayName, InContext->GetSender().ToString());

			bIsRecording = false;

			if (GetDeviceSettings<ULiveLinkUnrealDeviceSettings>()->bHasRecordStopAuthority)
			{
				ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
				const bool bWasRecording = Session.IsRecording();
				bool bStoppedRecording = false;
				if (bWasRecording)
				{
					bStoppedRecording = ILiveLinkRecordingSession::Get().StopRecording();
				}

				UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Has record stop authority; "
					"bWasRecording={bWasRecording}, bStoppedRecording={bStoppedRecording}",
					GetSettings()->DisplayName, bWasRecording, bStoppedRecording);
			}
		}
		else if (MessageTypeInfo == FLiveLinkTakeRecorderCmd_SetSlateName::StaticStruct())
		{
			const FLiveLinkTakeRecorderCmd_SetSlateName* Cmd =
				static_cast<const FLiveLinkTakeRecorderCmd_SetSlateName*>(InContext->GetMessage());

			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: SetSlateName(\"{SlateName}\") from {RemoteAddress}",
				GetSettings()->DisplayName, Cmd->SlateName, InContext->GetSender().ToString());

			if (GetDeviceSettings<ULiveLinkUnrealDeviceSettings>()->bHasTakeMetadataAuthority)
			{
				ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
				const FString PrevSlateName = Session.GetSlateName();

				bSuppressSlateNameSend = true;
				Session.SetSlateName(Cmd->SlateName);
				bSuppressSlateNameSend = false;

				UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Has take metadata authority; "
					"PrevSlateName={PrevSlateName}, NewSlateName={NewSlateName}",
					GetSettings()->DisplayName, PrevSlateName, Cmd->SlateName);
			}
		}
		else if (MessageTypeInfo == FLiveLinkTakeRecorderCmd_SetTakeNumber::StaticStruct())
		{
			const FLiveLinkTakeRecorderCmd_SetTakeNumber* Cmd =
				static_cast<const FLiveLinkTakeRecorderCmd_SetTakeNumber*>(InContext->GetMessage());

			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: SetTakeNumber({TakeNumber}) from {RemoteAddress}",
				GetSettings()->DisplayName, Cmd->TakeNumber, InContext->GetSender().ToString());

			if (GetDeviceSettings<ULiveLinkUnrealDeviceSettings>()->bHasTakeMetadataAuthority)
			{
				ILiveLinkRecordingSession& Session = ILiveLinkRecordingSession::Get();
				const int32 PrevTakeNumber = Session.GetTakeNumber();

				bSuppressTakeNumberSend = true;
				Session.SetTakeNumber(Cmd->TakeNumber);
				bSuppressTakeNumberSend = false;

				UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Has take metadata authority; "
					"PrevTakeNumber={PrevTakeNumber}, NewTakeNumber={NewTakeNumber}",
					GetSettings()->DisplayName, PrevTakeNumber, Cmd->TakeNumber);
			}
		}
	}
}


void ULiveLinkUnrealDevice::HandleMessageBusNotification(const FMessageBusNotification& InNotification)
{
	if (InNotification.NotificationType == EMessageBusNotification::Unregistered)
	{
		if (InNotification.RegistrationAddress == DeviceAddress)
		{
			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Message address unregistered {RemoteAddress}",
				GetSettings()->DisplayName, DeviceAddress.ToString());

			OnDisconnect();
		}
	}
}


void ULiveLinkUnrealDevice::Connect()
{
	if (ConnectionStatus != ELiveLinkDeviceConnectionStatus::Disconnected)
	{
		return;
	}

	const ULiveLinkUnrealDeviceSettings* DeviceSettings = GetDeviceSettings<ULiveLinkUnrealDeviceSettings>();
	if (DeviceSettings->ClientId.IsValid())
	{
		ILiveLinkHubClientsModel& ClientsModel = ILiveLinkHubClientsModel::GetChecked();

		UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "{DeviceName}: Connecting to {ClientName}",
			GetSettings()->DisplayName, ClientsModel.GetClientDisplayName(DeviceSettings->ClientId).ToString());

		MessageEndpoint = FMessageEndpointBuilder(GetFName())
			.WithCatchall(this, &ULiveLinkUnrealDevice::HandleMessageCatchall)
			.NotificationHandling(FOnBusNotification::CreateUObject(this, &ULiveLinkUnrealDevice::HandleMessageBusNotification))
			.ReceivingOnThread(ENamedThreads::GameThread)
			.Build();

		FLiveLinkUnrealDeviceAuxChannelRequestMessage* RequestMessage =
			FMessageEndpoint::MakeMessage<FLiveLinkUnrealDeviceAuxChannelRequestMessage>();

		ChannelId = FGuid::NewGuid();
		RequestMessage->ChannelId = ChannelId;

		ClientsModel.RequestAuxiliaryChannel(DeviceSettings->ClientId, *MessageEndpoint, *RequestMessage);

		SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Connecting);
	}
}


void ULiveLinkUnrealDevice::Disconnect()
{
	if (MessageEndpoint && DeviceAddress.IsValid() && ChannelId.IsValid())
	{
		FLiveLinkHubAuxChannelCloseMessage* CloseMessage =
			FMessageEndpoint::MakeMessage<FLiveLinkHubAuxChannelCloseMessage>();
		CloseMessage->ChannelId = ChannelId;
		MessageEndpoint->Send(CloseMessage, EMessageFlags::Reliable, {}, nullptr, { DeviceAddress }, FTimespan::Zero(), FDateTime::MaxValue());
	}

	OnDisconnect();
}


void ULiveLinkUnrealDevice::OnDisconnect()
{
	bIsRecording = false;

	ChannelId.Invalidate();
	DeviceAddress.Invalidate();
	MessageEndpoint.Reset();

	SetConnectionStatus(ELiveLinkDeviceConnectionStatus::Disconnected);
}


#undef LOCTEXT_NAMESPACE
