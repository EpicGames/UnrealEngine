// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkDevice.h"
#include "ILiveLinkHubClientsModel.h"
#include "IMessageContext.h"
#include "LiveLinkDeviceCapability_Connection.h"
#include "LiveLinkDeviceCapability_Recording.h"
#include "LiveLinkUnrealDevice.generated.h"


DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkUnrealDevice, Log, All);


struct FMessageBusNotification;
class FMessageEndpoint;
class IMessageContext;


/** Settings for an Unreal device instance. */
UCLASS()
class ULiveLinkUnrealDeviceSettings : public ULiveLinkDeviceSettings
{
	GENERATED_BODY()

public:

	ULiveLinkUnrealDeviceSettings()
	{
		DisplayName = NSLOCTEXT("LiveLinkUnrealDevice", "DefaultDisplayName", "Unreal Editor").ToString();
	}

	/** The editor client known to Live Link Hub to connect to. */
	UPROPERTY(EditAnywhere, Category="Unreal Device")
	FLiveLinkHubClientId ClientId;

	/** If true, a "recording started" event from this device will also cause Live Link Hub to begin recording. */
	UPROPERTY(EditAnywhere, Category="Unreal Device", AdvancedDisplay)
	bool bHasRecordStartAuthority = true;

	/** If true, a "recording stopped" event from this device will also cause Live Link Hub to stop recording. */
	UPROPERTY(EditAnywhere, Category="Unreal Device", AdvancedDisplay)
	bool bHasRecordStopAuthority = true;

	/** If true, slate/take changed events from this device will also cause Live Link Hub to update. 
	 * If true, this device will not accept updates to the slate/take sent from Live Link Hub.
	 */
	UPROPERTY(EditAnywhere, Category="Unreal Device", AdvancedDisplay, DisplayName="Has Slate Name & Take Number Authority")
	bool bHasTakeMetadataAuthority = true;
};


/**
 * A device for connecting to a running Unreal Editor instance.
 * 
 * Supports sending/receiving Take Recorder events to/from the remote editor session.
 */
UCLASS(meta=(DisplayName="Unreal"))
class ULiveLinkUnrealDevice : public ULiveLinkDevice
	, public ILiveLinkDeviceCapability_Connection
	, public ILiveLinkDeviceCapability_Recording
{
	GENERATED_BODY()

public:
	ULiveLinkUnrealDeviceSettings* GetSettings() { return GetDeviceSettings<ULiveLinkUnrealDeviceSettings>(); }
	const ULiveLinkUnrealDeviceSettings* GetSettings() const { return GetDeviceSettings<ULiveLinkUnrealDeviceSettings>(); }

	//~ Begin ULiveLinkDevice interface
public:
	virtual TSubclassOf<ULiveLinkDeviceSettings> GetSettingsClass() const override { return ULiveLinkUnrealDeviceSettings::StaticClass(); }

	virtual EDeviceHealth GetDeviceHealth() const override;
	virtual FText GetHealthText() const override;

	virtual void OnDeviceAdded() override;
	virtual void OnDeviceRemoved() override;
	virtual void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent) override;
	//~ End ULiveLinkDevice interface

	//~ Begin ILiveLinkDeviceCapability_Connection interface
public:
	virtual ELiveLinkDeviceConnectionStatus GetConnectionStatus_Implementation() const override;
	virtual FString GetHardwareId_Implementation() const override;
	virtual bool SetHardwareId_Implementation(const FString& InHardwareID) override;
	virtual bool Connect_Implementation() override;
	virtual bool Disconnect_Implementation() override;
protected:
	virtual void SetConnectionStatus(ELiveLinkDeviceConnectionStatus InStatus) override;
	//~ End ILiveLinkDeviceCapability_Connection interface

	//~ Begin ILiveLinkDeviceCapability_Recording interface
public:
	virtual bool StartRecording_Implementation() override;
	virtual bool StopRecording_Implementation() override;
	virtual bool IsRecording_Implementation() const override;
	//~ End ILiveLinkDeviceCapability_Recording interface

private:
	TPair<EDeviceHealth, FText> GetHealth() const;

	// Take Recorder event handlers.
	void HandleSlateNameChanged(FStringView InSlateName);
	void HandleTakeNumberChanged(int32 InTakeNumber);

	// Message bus event handlers.
	void HandleMessageCatchall(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleMessageBusNotification(const FMessageBusNotification& InNotification);

	/** Attempts to connect to the configured `ClientId`. */
	void Connect();

	/** Sends a close message connected editor client (if any). */
	void Disconnect();

	/** Called to perform cleanup and state transitions for both expected and unexpected disconnections. */
	void OnDisconnect();

private:
	/** Updated in response to internal state transitions, and reflected in the device UI. */
	ELiveLinkDeviceConnectionStatus ConnectionStatus = ELiveLinkDeviceConnectionStatus::Disconnected;

	/** The endpoint used for communicating with the editor. */
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	/** The address that accepted the aux channel request; we Send() here rather than Publish(). */
	FMessageAddress DeviceAddress;

	/** The aux channel identifier. */
	FGuid ChannelId;

	/** Recording state is updated in response to device event messages. */
	bool bIsRecording = false;

	/**
	 * Guards against feedback loops when receiving slate/take changes from UE.
	 * Set true while processing an incoming command so that the resulting
	 * ILiveLinkRecordingSession delegate doesn't re-send the same command back.
	 */
	bool bSuppressSlateNameSend = false;
	bool bSuppressTakeNumberSend = false;
};
