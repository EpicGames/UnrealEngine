// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubUnrealDeviceAux.h"
#include "Devices/LiveLinkUnrealDevice.h"
#include "Engine/Engine.h"
#include "ILiveLinkHubMessagingModule.h"
#include "LiveLinkUnrealDeviceMessages.h"
#include "Logging/StructuredLog.h"
#include "MessageEndpointBuilder.h"
#include "Recorder/TakeRecorderSubsystem.h"
#include "TakeMetaData.h"


FLiveLinkHubUnrealDeviceAuxManager::FLiveLinkHubUnrealDeviceAuxManager()
{
	FMessageEndpointBuilder EndpointBuilder = FMessageEndpointBuilder("LiveLinkHubTakeRecorderAuxHandler");
	MessageEndpoint = EndpointBuilder
		.Handling<FLiveLinkTakeRecorderCmd_SetSlateName>(this, &FLiveLinkHubUnrealDeviceAuxManager::HandleSetSlateName)
		.Handling<FLiveLinkTakeRecorderCmd_SetTakeNumber>(this, &FLiveLinkHubUnrealDeviceAuxManager::HandleSetTakeNumber)
		.Handling<FLiveLinkTakeRecorderCmd_StartRecording>(this, &FLiveLinkHubUnrealDeviceAuxManager::HandleStartRecording)
		.Handling<FLiveLinkTakeRecorderCmd_StopRecording>(this, &FLiveLinkHubUnrealDeviceAuxManager::HandleStopRecording)
		.Handling<FLiveLinkHubAuxChannelCloseMessage>(this, &FLiveLinkHubUnrealDeviceAuxManager::HandleAuxClose)
		.ReceivingOnThread(ENamedThreads::GameThread)
		.Build();

	RegisterRequestHandler();
	RegisterTakeRecorderDelegates();
}


FLiveLinkHubUnrealDeviceAuxManager::~FLiveLinkHubUnrealDeviceAuxManager()
{
	ILiveLinkHubMessagingModule* HubMessagingModule = FModuleManager::Get().GetModulePtr<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");
	if (HubMessagingModule)
	{
		HubMessagingModule->UnregisterAuxChannelRequestHandler<FLiveLinkUnrealDeviceAuxChannelRequestMessage>();
	}

	UTakeMetaData::OnTakeSlateChanged().RemoveAll(this);
	UTakeMetaData::OnTakeNumberChanged().RemoveAll(this);

	if (GEngine)
	{
		if (UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>())
		{
			TakeRecorder->GetOnRecordingStartedEvent().RemoveAll(this);
			TakeRecorder->GetOnRecordingStoppedEvent().RemoveAll(this);
		}
	}
}


void FLiveLinkHubUnrealDeviceAuxManager::RegisterRequestHandler()
{
	ILiveLinkHubMessagingModule& HubMessagingModule =
		FModuleManager::Get().GetModuleChecked<ILiveLinkHubMessagingModule>("LiveLinkHubMessaging");

	HubMessagingModule.RegisterAuxChannelRequestHandler<FLiveLinkUnrealDeviceAuxChannelRequestMessage>(
		[this]
		(
			const FLiveLinkUnrealDeviceAuxChannelRequestMessage& InRequest,
			const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
		)
		{
			UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Received Take Recorder channel request from {Sender}",
				InContext->GetSender().ToString());

			const FGuid& ChannelId = InRequest.ChannelId;

			if (FMessageAddress* ExistingAddress = ChannelToAddress.Find(ChannelId))
			{
				// Shouldn't happen.
				UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Duplicate channel ID {ChannelId} ({ExistingAddress})",
					ChannelId.ToString(), ExistingAddress->ToString());

				AddressToChannel.Remove(*ExistingAddress);
				ChannelToAddress.Remove(ChannelId);
			}

			if (FGuid* ExistingChannel = AddressToChannel.Find(InContext->GetSender()))
			{
				// Shouldn't happen.
				UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Duplicate aux address {SenderAddress} ({ExistingChannel})",
					InContext->GetSender().ToString(), ExistingChannel->ToString());

				ChannelToAddress.Remove(*ExistingChannel);
				AddressToChannel.Remove(InContext->GetSender());
			}

			ChannelToAddress.Add(ChannelId, InContext->GetSender());
			AddressToChannel.Add(InContext->GetSender(), ChannelId);

			FLiveLinkHubAuxChannelAcceptMessage* AcceptMessage =
				FMessageEndpoint::MakeMessage<FLiveLinkHubAuxChannelAcceptMessage>();
			AcceptMessage->ChannelId = ChannelId;
			MessageEndpoint->Send(AcceptMessage, EMessageFlags::Reliable, {}, nullptr, { InContext->GetSender() },
				FTimespan::Zero(), FDateTime::MaxValue());
		}
	);
}


void FLiveLinkHubUnrealDeviceAuxManager::RegisterTakeRecorderDelegates()
{
	if (UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>(); ensure(TakeRecorder))
	{
		TakeRecorder->GetOnRecordingStartedEvent().AddRaw(this, &FLiveLinkHubUnrealDeviceAuxManager::OnRecordingStarted);
		TakeRecorder->GetOnRecordingStoppedEvent().AddRaw(this, &FLiveLinkHubUnrealDeviceAuxManager::OnRecordingStopped);
	}

	UTakeMetaData::OnTakeSlateChanged().AddRaw(this, &FLiveLinkHubUnrealDeviceAuxManager::OnSlateNameChanged);
	UTakeMetaData::OnTakeNumberChanged().AddRaw(this, &FLiveLinkHubUnrealDeviceAuxManager::OnTakeNumberChanged);
}


bool FLiveLinkHubUnrealDeviceAuxManager::IsKnownSender(const FMessageAddress& InAddress) const
{
	if (AddressToChannel.Contains(InAddress))
	{
		return true;
	}

	UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Message from unknown sender {Sender} will be ignored", InAddress.ToString());
	return false;
}


void FLiveLinkHubUnrealDeviceAuxManager::HandleAuxClose(
	const FLiveLinkHubAuxChannelCloseMessage& InMessage,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Received channel close from {Sender}", InContext->GetSender().ToString());

	if (!IsKnownSender(InContext->GetSender()))
	{
		return;
	}

	if (FGuid* ExistingChannel = AddressToChannel.Find(InContext->GetSender()))
	{
		if (*ExistingChannel != InMessage.ChannelId)
		{
			UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Channel close has wrong ID {ChannelId} (expected {ExistingId})",
				InMessage.ChannelId.ToString(), ExistingChannel->ToString());
		}

		AddressToChannel.Remove(InContext->GetSender());
	}
	else
	{
		// Shouldn't happen.
		UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Channel with address {Sender} not found", InContext->GetSender().ToString());
	}

	if (FMessageAddress* ExistingAddress = ChannelToAddress.Find(InMessage.ChannelId))
	{
		if (*ExistingAddress != InContext->GetSender())
		{
			UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Channel close not from expected sender {ExistingAddress}",
				ExistingAddress->ToString());
		}

		ChannelToAddress.Remove(InMessage.ChannelId);
	}
	else
	{
		// Shouldn't happen.
		UE_LOGFMT(LogLiveLinkUnrealDevice, Warning, "Channel with ID {ChannelId} not found", InMessage.ChannelId.ToString());
	}
}


void FLiveLinkHubUnrealDeviceAuxManager::HandleSetSlateName(
	const FLiveLinkTakeRecorderCmd_SetSlateName& InCmd,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Received SetSlateName from {Sender}", InContext->GetSender().ToString());

	if (!IsKnownSender(InContext->GetSender()))
	{
		return;
	}

	if (UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>(); ensure(TakeRecorder))
	{
		// Note the sender so that the resulting OnSlateNameChanged broadcast excludes it.
		SlateNameChangeSource = InContext->GetSender();
		TakeRecorder->SetSlateName(InCmd.SlateName);
		SlateNameChangeSource.Reset();
	}
}


void FLiveLinkHubUnrealDeviceAuxManager::HandleSetTakeNumber(
	const FLiveLinkTakeRecorderCmd_SetTakeNumber& InCmd,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Received SetTakeNumber from {Sender}", InContext->GetSender().ToString());

	if (!IsKnownSender(InContext->GetSender()))
	{
		return;
	}

	if (UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>(); ensure(TakeRecorder))
	{
		// Note the sender so that the resulting OnTakeNumberChanged broadcast excludes it.
		TakeNumberChangeSource = InContext->GetSender();
		TakeRecorder->SetTakeNumber(InCmd.TakeNumber);
		TakeNumberChangeSource.Reset();
	}
}


void FLiveLinkHubUnrealDeviceAuxManager::HandleStartRecording(
	const FLiveLinkTakeRecorderCmd_StartRecording& InCmd,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Received StartRecording from {Sender}", InContext->GetSender().ToString());

	if (!IsKnownSender(InContext->GetSender()))
	{
		return;
	}

	if (UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>(); ensure(TakeRecorder))
	{
		// Note the sender so any slate/take delegate fired synchronously by
		// StartRecording doesn't echo straight back to the originator.
		SlateNameChangeSource = InContext->GetSender();
		TakeNumberChangeSource = InContext->GetSender();
		TakeRecorder->StartRecording();
		SlateNameChangeSource.Reset();
		TakeNumberChangeSource.Reset();
	}
}


void FLiveLinkHubUnrealDeviceAuxManager::HandleStopRecording(
	const FLiveLinkTakeRecorderCmd_StopRecording& InCmd,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext
)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Received StopRecording from {Sender}", InContext->GetSender().ToString());

	if (!IsKnownSender(InContext->GetSender()))
	{
		return;
	}

	if (UTakeRecorderSubsystem* TakeRecorder = GEngine->GetEngineSubsystem<UTakeRecorderSubsystem>(); ensure(TakeRecorder))
	{
		// Note the sender so any slate/take delegate fired synchronously by
		// StopRecording (e.g. post-stop auto-increment) doesn't echo straight
		// back to the originator.
		SlateNameChangeSource = InContext->GetSender();
		TakeNumberChangeSource = InContext->GetSender();
		TakeRecorder->StopRecording();
		SlateNameChangeSource.Reset();
		TakeNumberChangeSource.Reset();
	}
}


void FLiveLinkHubUnrealDeviceAuxManager::OnRecordingStarted(UTakeRecorder* InRecorder)
{
	FLiveLinkTakeRecorderEvent_RecordingStarted* EventMessage = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderEvent_RecordingStarted>();
	BroadcastToChannels(EventMessage, FLiveLinkTakeRecorderEvent_RecordingStarted::StaticStruct());
}


void FLiveLinkHubUnrealDeviceAuxManager::OnRecordingStopped(UTakeRecorder* InRecorder)
{
	FLiveLinkTakeRecorderEvent_RecordingStopped* EventMessage = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderEvent_RecordingStopped>();
	BroadcastToChannels(EventMessage, FLiveLinkTakeRecorderEvent_RecordingStopped::StaticStruct());
}


void FLiveLinkHubUnrealDeviceAuxManager::OnSlateNameChanged(const FString& InSlateName, UTakeMetaData* InTakeMetaData)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Slate name changed to \"{SlateName}\", broadcasting to channels",
		InSlateName);

	FLiveLinkTakeRecorderCmd_SetSlateName* Message = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderCmd_SetSlateName>();
	Message->SlateName = InSlateName;
	BroadcastToChannels(Message, FLiveLinkTakeRecorderCmd_SetSlateName::StaticStruct(), SlateNameChangeSource);
}


void FLiveLinkHubUnrealDeviceAuxManager::OnTakeNumberChanged(int32 InTakeNumber, UTakeMetaData* InTakeMetaData)
{
	UE_LOGFMT(LogLiveLinkUnrealDevice, Log, "Take number changed to {TakeNumber}, broadcasting to channels",
		InTakeNumber);

	FLiveLinkTakeRecorderCmd_SetTakeNumber* Message = FMessageEndpoint::MakeMessage<FLiveLinkTakeRecorderCmd_SetTakeNumber>();
	Message->TakeNumber = InTakeNumber;
	BroadcastToChannels(Message, FLiveLinkTakeRecorderCmd_SetTakeNumber::StaticStruct(), TakeNumberChangeSource);
}


void FLiveLinkHubUnrealDeviceAuxManager::BroadcastToChannels(
	void* InMessage,
	UScriptStruct* InTypeInfo,
	const TOptional<FMessageAddress>& InExclude
)
{
	TArray<FMessageAddress> Recipients;
	Recipients.Reserve(ChannelToAddress.Num());

	for (const TPair<FGuid, FMessageAddress>& Pair : ChannelToAddress)
	{
		if (!InExclude.IsSet() || Pair.Value != InExclude.GetValue())
		{
			Recipients.Add(Pair.Value);
		}
	}

	MessageEndpoint->Send(InMessage, InTypeInfo, EMessageFlags::Reliable, {}, nullptr, Recipients,
		FTimespan::Zero(), FDateTime::MaxValue());
}
