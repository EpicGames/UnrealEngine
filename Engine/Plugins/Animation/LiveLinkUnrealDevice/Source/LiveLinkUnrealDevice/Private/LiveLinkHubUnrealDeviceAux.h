// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"


struct FLiveLinkHubAuxChannelCloseMessage;
struct FLiveLinkTakeRecorderCmd_SetSlateName;
struct FLiveLinkTakeRecorderCmd_SetTakeNumber;
struct FLiveLinkTakeRecorderCmd_StartRecording;
struct FLiveLinkTakeRecorderCmd_StopRecording;
class FMessageEndpoint;
class IMessageContext;
class UTakeMetaData;
class UTakeRecorder;


/**
 * Configures a Live Link Hub auxiliary message channel for communicating with Unreal devices.
 *
 * Handles incoming Take Recorder commands, and relays events to connected devices.
 */
class FLiveLinkHubUnrealDeviceAuxManager
{
public:
	FLiveLinkHubUnrealDeviceAuxManager();
	~FLiveLinkHubUnrealDeviceAuxManager();

private:
	// Init helpers
	void RegisterRequestHandler();
	void RegisterTakeRecorderDelegates();

	// Take Recorder event handlers
	void OnRecordingStarted(UTakeRecorder* InRecorder);
	void OnRecordingStopped(UTakeRecorder* InRecorder);
	void OnSlateNameChanged(const FString& InSlateName, UTakeMetaData* InTakeMetaData);
	void OnTakeNumberChanged(int32 InTakeNumber, UTakeMetaData* InTakeMetaData);

	// Message handlers
	bool IsKnownSender(const FMessageAddress& InAddress) const;

	void HandleAuxClose(const FLiveLinkHubAuxChannelCloseMessage& InMessage, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	void HandleSetSlateName(const FLiveLinkTakeRecorderCmd_SetSlateName& InCmd, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleSetTakeNumber(const FLiveLinkTakeRecorderCmd_SetTakeNumber& InCmd, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleStartRecording(const FLiveLinkTakeRecorderCmd_StartRecording& InCmd, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);
	void HandleStopRecording(const FLiveLinkTakeRecorderCmd_StopRecording& InCmd, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InContext);

	/** Broadcast a message to all connected channels, optionally excluding one address. */
	void BroadcastToChannels(void* InMessage, UScriptStruct* InTypeInfo, const TOptional<FMessageAddress>& InExclude = {});

private:
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	TMap<FGuid, FMessageAddress> ChannelToAddress;
	TMap<FMessageAddress, FGuid> AddressToChannel;

	/**
	 * When processing an incoming slate/take change from a remote device, we note the sender
	 * address here so that the corresponding outgoing broadcast (triggered by the local delegate)
	 * can exclude that sender and avoid a feedback loop.
	 */
	TOptional<FMessageAddress> SlateNameChangeSource;
	TOptional<FMessageAddress> TakeNumberChangeSource;
};
