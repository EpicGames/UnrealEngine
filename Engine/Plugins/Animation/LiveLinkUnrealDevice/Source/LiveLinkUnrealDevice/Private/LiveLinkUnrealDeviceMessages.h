// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "LiveLinkHubMessages.h"
#include "Misc/Timecode.h"
#include "LiveLinkUnrealDeviceMessages.generated.h"


USTRUCT()
struct FLiveLinkUnrealDeviceAuxChannelRequestMessage : public FLiveLinkHubAuxChannelRequestMessage
{
	GENERATED_BODY()
};


USTRUCT()
struct FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	static constexpr uint8 CurrentVersion = 1;

	UPROPERTY()
	uint8 MessageVersion = CurrentVersion;

	UPROPERTY()
	FGuid MessageId;
};


USTRUCT()
struct FLiveLinkTakeRecorderSlateInfo
{
	GENERATED_BODY()

	UPROPERTY()
	TOptional<FString> SlateName;

	UPROPERTY()
	TOptional<int32> TakeNumber;

	UPROPERTY()
	TOptional<FString> Description;
};


//////////////////////////////////////////////////////////////////////////


USTRUCT()
struct FLiveLinkTakeRecorderCmd_SetSlateName : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FString SlateName;
};


USTRUCT()
struct FLiveLinkTakeRecorderCmd_SetTakeNumber : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	int32 TakeNumber = 0;
};


USTRUCT()
struct FLiveLinkTakeRecorderCmd_StartRecording : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FLiveLinkTakeRecorderSlateInfo SlateInfo;
};


USTRUCT()
struct FLiveLinkTakeRecorderCmd_StopRecording : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()
};


//////////////////////////////////////////////////////////////////////////


USTRUCT()
struct FLiveLinkTakeRecorderEvent_RecordingStarting : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	float CountdownSec = 0.0;
};


USTRUCT()
struct FLiveLinkTakeRecorderEvent_RecordingStarted : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FTimecode StartTimecode;
};


USTRUCT()
struct FLiveLinkTakeRecorderEvent_RecordingStopped : public FLiveLinkTakeRecorderMessageBase
{
	GENERATED_BODY()

	UPROPERTY()
	FTimecode EndTimecode;

	UPROPERTY()
	bool bCancelled = false;
};
