// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkHubUnrealDeviceAux.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"


class FLiveLinkUnrealDeviceModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual ~FLiveLinkUnrealDeviceModule() = default;

	virtual void StartupModule()
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda(
			[this]()
			{
				UnrealDeviceAuxManager = MakeUnique<FLiveLinkHubUnrealDeviceAuxManager>();
			}
		);
	}

	virtual void PreUnloadCallback()
	{
		UnrealDeviceAuxManager.Reset();
	}
	//~ End IModuleInterface

private:
	TUniquePtr<FLiveLinkHubUnrealDeviceAuxManager> UnrealDeviceAuxManager;
};


IMPLEMENT_MODULE(FLiveLinkUnrealDeviceModule, LiveLinkUnrealDevice);
