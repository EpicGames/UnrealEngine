// Copyright Epic Games, Inc. All Rights Reserved.

#include "Devices/LiveLinkUnrealDevice.h"
#include "Misc/AutomationTest.h"


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUnrealDeviceTest, "LiveLinkHub.UnrealDevice",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)


bool FUnrealDeviceTest::RunTest(const FString& Parameters)
{
	ULiveLinkUnrealDevice* Device = NewObject<ULiveLinkUnrealDevice>();

	return true;
}
